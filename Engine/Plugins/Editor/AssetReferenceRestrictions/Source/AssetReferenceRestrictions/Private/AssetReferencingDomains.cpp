// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetReferencingDomains.h"
#include "AssetReferencingPolicySettings.h"
#include "Interfaces/IPluginManager.h"
#include "DomainAssetReferenceFilter.h"
#include "Editor.h"
#include "Misc/PackageName.h"
#include "Settings/ProjectPackagingSettings.h"
#include "Misc/Paths.h"
#include "PluginReferenceDescriptor.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"

DEFINE_LOG_CATEGORY(LogAssetReferenceRestrictions);

#define LOCTEXT_NAMESPACE "AssetReferencingPolicy"

//////////////////////////////////////////////////////////////////////
// FDomainPathNode

// Struct used to accelerate root path matching
struct FDomainPathNode final : public TSharedFromThis<FDomainPathNode>
{
public:
	// The map of sub-folders to children nodes that contain a more specific domain
	// (each sub-folder is of the form SubfolderName, with no leading or trailing slashes)
	TMap<FString, TSharedPtr<FDomainPathNode>> SubFolders;

	// The domain to use if nothing more specific in SubFolders matches
	TSharedPtr<FDomainData> DefaultDomain;
	
	// Weak pointer to parent node for debugging 
	TWeakPtr<FDomainPathNode> Parent;
	
	// Path segment for this node for debugging
	FString PathSegment;

public:
	
	FDomainPathNode(TWeakPtr<FDomainPathNode> InParent, FString InPathSegment)
		: Parent(InParent)
		, PathSegment(InPathSegment)
	{

	}

	// RemainingPath is expected to have leading but no trailing /
	TSharedPtr<FDomainData> FindDomainFromPath(FStringView RemainingPath) const
	{
		// Take a /FirstDir/SecondDir/ThirdDir path and isolate FirstDir into DirectoryName
		FStringView DirectoryName = RemainingPath.Mid(1);

		int32 PathSeparatorIndex = INDEX_NONE;
		if (DirectoryName.FindChar(TEXT('/'), /*out*/ PathSeparatorIndex))
		{
			DirectoryName = DirectoryName.Left(PathSeparatorIndex);
		}

		// If DirectoryName is a known sub-folder, recurse into it
		if (DirectoryName.Len() > 0)
		{
			if (const TSharedPtr<FDomainPathNode>* pChildFolder = SubFolders.FindByHash(GetTypeHash(DirectoryName), DirectoryName))
			{
				const FStringView PathRelativeToDirectoryName(RemainingPath.Mid(DirectoryName.Len() + 1));

				TSharedPtr<FDomainData> ChildResult = (*pChildFolder)->FindDomainFromPath(PathRelativeToDirectoryName);
				if (ChildResult.IsValid())
				{
					return ChildResult;
				}
			}
		}

		// We've gotten this far and it doesn't match any more specific sub-folders, so return the leaf value
		// (which might be nullptr if we were a sub-folder that didn't match all the way)
		return DefaultDomain;
	}
	
	void AddDomain(TSharedPtr<FDomainData> Domain, FStringView RemainingPath)
	{
		int32 DirectorySeparatorIndex;
		if (RemainingPath.FindChar(TEXT('/'), /*out*/ DirectorySeparatorIndex))
		{
			check(DirectorySeparatorIndex > 0);
			const FString DirectoryName = FString::ConstructFromPtrSize(RemainingPath.GetData(), DirectorySeparatorIndex);

			TSharedPtr<FDomainPathNode>& ChildFolder = SubFolders.FindOrAdd(DirectoryName);
			if (!ChildFolder.IsValid())
			{
				ChildFolder = MakeShared<FDomainPathNode>(this->AsWeak(), DirectoryName);
			}

			ChildFolder->AddDomain(Domain, RemainingPath.Mid(DirectorySeparatorIndex + 1));
		}
		else
		{
			UE_LOGF(LogAssetReferenceRestrictions, Verbose, "Assigning path %ls to domain %ls", *GetPath(), *Domain->UserFacingDomainName.ToString());
			check(RemainingPath.Len() == 0);
			ensureMsgf(!DefaultDomain.IsValid(), TEXT("Registered the path %s for two different domains: %s and %s"), 
				*GetPath(), 
				*DefaultDomain->UserFacingDomainName.ToString(),
				*Domain->UserFacingDomainName.ToString()
				);
			DefaultDomain = Domain;
		}
	}
	
	FString GetPath() const
	{
		if (TSharedPtr<FDomainPathNode> PinnedParent = Parent.Pin(); PinnedParent.IsValid())
		{
			return FString::Printf(TEXT("%s%s/"), *PinnedParent->GetPath(), *PathSegment);
		}
		else
		{
			return FString(TEXT("/"));
		}
	}

	void DebugPrint(const FString& PreceedingPath, int32 Depth) const
	{
		FString DomainSuffix;
		if (DefaultDomain.IsValid())
		{
			DomainSuffix = TEXT(" ---> ") + DefaultDomain->UserFacingDomainName.ToString();
		}

		FString ChildrenSuffix;
		if (SubFolders.Num() > 0)
		{
			ChildrenSuffix = FString::Printf(TEXT(" has %d children"), SubFolders.Num());
		}

		UE_LOGF(LogAssetReferenceRestrictions, Log, "[%d] Path %ls%ls%ls", Depth, *PreceedingPath, *ChildrenSuffix, *DomainSuffix);

		for (const auto& KVP : SubFolders)
		{
			KVP.Value->DebugPrint(PreceedingPath / KVP.Key, Depth + 1);
		}
	}
};

//////////////////////////////////////////////////////////////////////
// FDomainDatabase

FDomainDatabase::FDomainDatabase()
{
}

FDomainDatabase::~FDomainDatabase()
{
	IPluginManager& PluginManager = IPluginManager::Get();
	PluginManager.OnNewPluginCreated().RemoveAll(this);
	PluginManager.OnNewPluginMounted().RemoveAll(this);
	PluginManager.OnPluginEdited().RemoveAll(this);
}

void FDomainDatabase::RebuildFromScratch()
{
	const UAssetReferencingPolicySettings* Settings = GetDefault<UAssetReferencingPolicySettings>();

	// Reset everything
	DomainNameMap.Reset();
	PathMap.Reset();
	EngineDomain.Reset();
	ScriptDomain.Reset();
	TempDomain.Reset();
	GameDomain.Reset();
	NeverCookDomain.Reset();
	DomainsDefinedByPlugins.Reset();
	PluginDomains.Reset();

	SpecificAssetPackageDomains.Reset();

	// Create the built-in domains
	//@TODO: Would be good to get the path roots directly from FLongPackagePathsSingleton but it's private
	EngineDomain = FindOrAddDomainByName(UAssetReferencingPolicySettings::EngineDomainName);
	EngineDomain->UserFacingDomainName = LOCTEXT("EngineDomain", "EngineContent");
	EngineDomain->DomainRootPaths.Add(TEXT("/Engine/"));

	ScriptDomain = FindOrAddDomainByName(UAssetReferencingPolicySettings::ScriptDomainName);
	ScriptDomain->UserFacingDomainName = LOCTEXT("ScriptDomain", "Script");
	ScriptDomain->bCanBeSeenByEverything = true;
	ScriptDomain->bCanSeeEverything = true;
	ScriptDomain->DomainRootPaths.Add(TEXT("/Script/"));

	TempDomain = FindOrAddDomainByName(UAssetReferencingPolicySettings::TempDomainName);
	TempDomain->UserFacingDomainName = LOCTEXT("TempDomain", "Temp");
	TempDomain->bCanSeeEverything = true;
	TempDomain->DomainRootPaths.Add(TEXT("/Temp/"));
	TempDomain->DomainRootPaths.Add(TEXT("/Extra/"));
	TempDomain->DomainRootPaths.Add(TEXT("/Memory/"));

	GameDomain = FindOrAddDomainByName(UAssetReferencingPolicySettings::GameDomainName);
	GameDomain->UserFacingDomainName = LOCTEXT("GameDomain", "ProjectContent");  	//@TODO: FText::AsCultureInvariant(GetDefault<UGeneralProjectSettings>()->ProjectName;);
	GameDomain->DomainRootPaths.Add(TEXT("/Game/"));
	
	GameDomain->DomainsVisibleFromHere.Add(EngineDomain);
	AddDomainVisibilityList(GameDomain, Settings->DefaultProjectContentRule.CanReferenceTheseDomains);

	// Create the rule driven folder-based domains
	for (const FARPDomainDefinitionByContentRoot& PathRule : Settings->AdditionalDomains)
	{
		if (PathRule.IsValid())
		{
			TSharedPtr<FDomainData> Domain = FindOrAddDomainByName(PathRule.DomainName);

			Domain->UserFacingDomainName = PathRule.DomainDisplayName;
			Domain->ErrorMessageIfUsedElsewhere = PathRule.ErrorMessageIfUsedElsewhere;

			for (const FDirectoryPath& PathRoot : PathRule.ContentRoots)
			{
				if (PathRoot.Path.StartsWith(TEXT("/"), ESearchCase::CaseSensitive))
				{
					Domain->DomainRootPaths.Add(PathRoot.Path / TEXT(""));
				}
			}

			for(FName PackageName : PathRule.SpecificAssets)
			{
				if(FPackageName::IsValidLongPackageName(PackageName.ToString()))
				{
					Domain->SpecificAssetPackages.Add(PackageName);
				}
				else
				{
					UE_LOGF(LogAssetReferenceRestrictions, Error, "Invalid specific asset path '%ls' for domain '%ls'", *PackageName.ToString(), *PathRule.DomainName);
				}
			}

			Domain->DomainsVisibleFromHere.Add(EngineDomain);

			if (PathRule.ReferenceMode == EARPDomainAllowedToReferenceMode::AllDomains)
			{
				Domain->bCanSeeEverything = true;
			}
			else
			{
				AddDomainVisibilityList(Domain, PathRule.CanReferenceTheseDomains);
			}
		}
	}

	// Create the domains for plugins that contain content
	TArray<TSharedRef<IPlugin>> EnabledPlugins = IPluginManager::Get().GetEnabledPlugins();
	for (const TSharedRef<IPlugin>& Plugin : EnabledPlugins)
	{
		// Temporary until NoGameFeatureContent is removed. Don't add dependencies for GFPs that have no content
		// other than the GameFeatureData
		bool bNoGameFeatureContent = false;
		Plugin->GetDescriptorJson()->TryGetBoolField(TEXT("NoGameFeatureContent"), bNoGameFeatureContent);
		
		if (bNoGameFeatureContent)
		{
			BuildUnrestrictedDomainFromPlugin(Plugin);
		}
		else if (Plugin->CanContainContent() || Plugin->CanContainVerse())
		{
			BuildDomainFromPlugin(Plugin);
		}
	}

	const UProjectPackagingSettings* const PackagingSettings = GetDefault<UProjectPackagingSettings>();
	check(PackagingSettings);
	if (PackagingSettings->DirectoriesToNeverCook.Num() > 0)
	{
		NeverCookDomain = FindOrAddDomainByName(UAssetReferencingPolicySettings::NeverCookDomainName);
		NeverCookDomain->UserFacingDomainName = LOCTEXT("NeverCook", "Never Cooked Content");
		NeverCookDomain->bCanSeeEverything = true;

		TSet<FString> DomainRootPaths;
		for (const FDirectoryPath& DirectoryToNeverCook : PackagingSettings->DirectoriesToNeverCook)
		{
			const FString UncookedFolder = DirectoryToNeverCook.Path.StartsWith(TEXT("/"), ESearchCase::CaseSensitive) ? DirectoryToNeverCook.Path : (TEXT("/Game/") + DirectoryToNeverCook.Path);
			DomainRootPaths.Add(UncookedFolder / TEXT(""));
		}
		NeverCookDomain->DomainRootPaths = DomainRootPaths.Array();
	}

	// Rebuild the path map
	PathMap = MakeShared<FDomainPathNode>(nullptr, FString{});
	for (const auto& KVP : DomainNameMap)
	{
		TSharedPtr<FDomainData> Domain = KVP.Value;
		for (const FString& DomainRootPath : Domain->DomainRootPaths)
		{
			check(DomainRootPath.StartsWith(TEXT("/"), ESearchCase::CaseSensitive) && DomainRootPath.EndsWith(TEXT("/"), ESearchCase::CaseSensitive));

			// Trim the leading /
			FStringView TrimmedRootPath(FStringView(DomainRootPath).Mid(1));

			PathMap->AddDomain(Domain, TrimmedRootPath);
		}

		for (FName PackageName : Domain->SpecificAssetPackages)
		{
			if (TSharedPtr<FDomainData>* ExistingDomain = SpecificAssetPackageDomains.Find(PackageName))
			{
				UE_LOGF(LogAssetReferenceRestrictions, Warning, "Overriding existing specific domain '%ls' of package '%ls' with new doman '%ls'", *(*ExistingDomain)->UserFacingDomainName.ToString(), *PackageName.ToString(), *Domain->UserFacingDomainName.ToString());
			}
			SpecificAssetPackageDomains.Add(PackageName, Domain);
		}
	}

	ValidateAllDomains();

	// Update any existing filters that have cached now stale domains
	FDomainAssetReferenceFilter::UpdateAllFilters();

	OnPostDatabaseUpdated.Broadcast();
}

void FDomainDatabase::Init()
{
	RebuildFromScratch();

#if UE_ASSET_DOMAIN_FILTERING_DEBUG_LOGGING
	DebugPrintAllDomains();
	UE_LOGF(LogAssetReferenceRestrictions, Log, "Finished building domain database");
#endif

	IPluginManager& PluginManager = IPluginManager::Get();
	PluginManager.OnNewPluginCreated().AddRaw(this, &FDomainDatabase::OnPluginCreatedOrMounted);
	PluginManager.OnNewPluginMounted().AddRaw(this, &FDomainDatabase::OnPluginCreatedOrMounted);
	PluginManager.OnPluginEdited().AddRaw(this, &FDomainDatabase::OnPluginCreatedOrMounted);
	
}

void FDomainDatabase::MarkDirty()
{
	if (!bDatabaseOutOfDate)
	{
		bDatabaseOutOfDate = true;

		if (GEditor->IsTimerManagerValid())
		{
			GEditor->GetTimerManager()->SetTimerForNextTick([this]() { UpdateIfNecessary(); });
		}
	}
}

void FDomainDatabase::UpdateIfNecessary()
{
	if (bDatabaseOutOfDate)
	{
		UE_LOGF(LogAssetReferenceRestrictions, Display, "Updating asset referencing domain DB due to plugin or rule changes");

		RebuildFromScratch();

		bDatabaseOutOfDate = false;
	}
}

void FDomainDatabase::OnPluginCreatedOrMounted(IPlugin& NewPlugin)
{
	MarkDirty();
}

TSharedPtr<FDomainData> FDomainDatabase::FindOrAddDomainByName(const FString& Name)
{
	TSharedPtr<FDomainData>& Result = DomainNameMap.FindOrAdd(Name);
	if (!Result.IsValid())
	{
		Result = MakeShared<FDomainData>();
	}
	return Result;
}

void FDomainDatabase::BuildDomainFromPlugin(TSharedRef<IPlugin> Plugin)
{
	const FString NewDomainName = Plugin->GetName();
	DomainsDefinedByPlugins.Add(NewDomainName);

	TSharedPtr<FDomainData> Domain = FindOrAddDomainByName(NewDomainName);
	PluginDomains.Add(Domain);

	Domain->Reset();
	Domain->DomainRootPaths.Add(Plugin->GetMountedAssetPath());

	// The plugin path starts like ../../../MyGamePath/MyGameName/Plugins/ThePathRelativeToPluginsDir/PluginName
	// And we want to extract just ThePathRelativeToPluginsDir, with the leading and trailing slashes
	const FString PluginsFolderRoot = (Plugin->GetLoadedFrom() == EPluginLoadedFrom::Project) ? FPaths::ProjectPluginsDir() : FPaths::EnginePluginsDir();
	FString PluginPathRelativeToDomain = Plugin->GetBaseDir();
	if (FPaths::MakePathRelativeTo(/*inout*/ PluginPathRelativeToDomain, *PluginsFolderRoot))
	{
		PluginPathRelativeToDomain = TEXT("/") + PluginPathRelativeToDomain + TEXT("/");
	}
	else
	{
		PluginPathRelativeToDomain = FString();
	}

	// Find the appropriate rule
	const UAssetReferencingPolicySettings* Settings = GetDefault<UAssetReferencingPolicySettings>();
	const FARPDomainSettingsForPlugins& DomainSettings = (Plugin->GetLoadedFrom() == EPluginLoadedFrom::Project) ? Settings->ProjectPlugins : Settings->EnginePlugins;

	const FARPDomainDefinitionForMatchingPlugins* BestSpecificRule = nullptr;
	for (const FARPDomainDefinitionForMatchingPlugins& TestRule : DomainSettings.AdditionalRules)
	{
		if (TestRule.IsValid())
		{
			if ((TestRule.MatchRule == EARPPluginMatchMode::MatchByCategory) && Plugin->GetDescriptor().Category.StartsWith(TestRule.PluginCategoryPrefix))
			{
				// It's a match, see if it's the best match
				if ((BestSpecificRule == nullptr) || 
					((BestSpecificRule->MatchRule == EARPPluginMatchMode::MatchByCategory) && (BestSpecificRule->PluginCategoryPrefix.Len() < TestRule.PluginCategoryPrefix.Len())))
				{
					BestSpecificRule = &TestRule;
				}
			}
			else if ((TestRule.MatchRule == EARPPluginMatchMode::MatchByPathPrefix) && PluginPathRelativeToDomain.StartsWith(TestRule.PluginPathPrefix))
			{
				if ((BestSpecificRule == nullptr) || 
					(BestSpecificRule->MatchRule == EARPPluginMatchMode::MatchByCategory) ||
					((BestSpecificRule->MatchRule == EARPPluginMatchMode::MatchByPathPrefix) && (BestSpecificRule->PluginPathPrefix.Len() < TestRule.PluginPathPrefix.Len())))
				{
					BestSpecificRule = &TestRule;
				}
			}
		}
	}

	// Fill out the domain info for this plugin from the template/rule
	Domain->UserFacingDomainName = FText::Format((BestSpecificRule != nullptr) ? BestSpecificRule->DisplayName : LOCTEXT("PluginDomainName", "Plugin:{0}"), FText::AsCultureInvariant(Plugin->GetName()));

	Domain->DomainsVisibleFromHere.Add(EngineDomain);

	if (BestSpecificRule != nullptr)
	{
		Domain->ErrorMessageIfUsedElsewhere = BestSpecificRule->ErrorMessageIfUsedElsewhere;
		AddDomainVisibilityList(Domain, BestSpecificRule->CanReferenceTheseDomains);
	}
	else
	{
		AddDomainVisibilityList(Domain, DomainSettings.DefaultRule.CanReferenceTheseDomains);

		if (DomainSettings.DefaultRule.bCanBeSeenByOtherDomainsWithoutDependency)
		{
			Domain->bCanBeSeenByEverything = true;
		}
		else if (DomainSettings.DefaultRule.bCanProjectAccessThesePlugins)
		{
			GameDomain->DomainsVisibleFromHere.Add(Domain);
		}
	}

	// Explicitly referenced dependencies are also always visible
	for (const FPluginReferenceDescriptor& Dependencies : Plugin->GetDescriptor().Plugins)
	{
		if (Dependencies.bEnabled)
		{
			TSharedPtr<IPlugin> PluginWeDependOn = IPluginManager::Get().FindEnabledPlugin(Dependencies.Name);
			if (PluginWeDependOn.IsValid() && (PluginWeDependOn->CanContainContent() || PluginWeDependOn->CanContainVerse()))
			{
				Domain->DomainsVisibleFromHere.Add(FindOrAddDomainByName(Dependencies.Name));
			}
		}
	}
}

void FDomainDatabase::BuildUnrestrictedDomainFromPlugin(TSharedRef<IPlugin> Plugin)
{
	const FString NewDomainName = Plugin->GetName();
	DomainsDefinedByPlugins.Add(NewDomainName);

	TSharedPtr<FDomainData> Domain = FindOrAddDomainByName(NewDomainName);
	PluginDomains.Add(Domain);

	Domain->Reset();
	Domain->DomainRootPaths.Add(Plugin->GetMountedAssetPath());
	Domain->bCanBeSeenByEverything = true;
}

void FDomainDatabase::ValidateAllDomains()
{
	for (const auto& KVP : DomainNameMap)
	{
		TSharedPtr<FDomainData> TestDomain = KVP.Value;
		if (!TestDomain->IsValid())
		{
			// Determine the referencing set
			TArray<FString> DomainsThatReferenceMe;
			for (const auto& RefKVP : DomainNameMap)
			{
				TSharedPtr<FDomainData> RefDomain = RefKVP.Value;
				if (RefDomain->DomainsVisibleFromHere.Contains(TestDomain))
				{
					DomainsThatReferenceMe.Add(RefKVP.Key);
				}
			}

			UE_LOGF(LogAssetReferenceRestrictions, Error, "Asset domain %ls was referenced by [%ls] but wasn't found (perhaps it is a plugin that was disabled?)", *KVP.Key, *FString::Join(DomainsThatReferenceMe, TEXT(", ")));
		}
	}
}

void FDomainDatabase::DebugPrintAllDomains()
{
	for (const auto& KVP : DomainNameMap)
	{
		TSharedPtr<FDomainData> Domain = KVP.Value;
		UE_LOGF(LogAssetReferenceRestrictions, Log, "Domain: %ls (%ls)", *KVP.Key, *Domain->UserFacingDomainName.ToString());
		UE_LOGF(LogAssetReferenceRestrictions, Log, "Roots: %ls", *FString::Join(Domain->DomainRootPaths, TEXT(", ")));
		UE_LOGF(LogAssetReferenceRestrictions, Log, "CanSee: %ls", *FString::JoinBy(Domain->DomainsVisibleFromHere, TEXT(", "), [](TSharedPtr<FDomainData> VisibleDomain) { return VisibleDomain->UserFacingDomainName.ToString(); }));
		UE_LOGF(LogAssetReferenceRestrictions, Log, "");
	}

	PathMap->DebugPrint(TEXT("/"), 0);
}

TSharedPtr<FDomainData> FDomainDatabase::FindDomainFromAssetData(const FAssetData& AssetData) const
{
	TSharedPtr<FDomainData> DomainResult = SpecificAssetPackageDomains.FindRef(AssetData.PackageName);
	
	if(!DomainResult.IsValid())
	{
		FString PackagePath = AssetData.PackagePath.ToString();

		// Treat external objects to be part of the same domain as their owning objects, so we don't treat them as being part of /Game/... just 
		// because they are in /Game/ExternalObjects/xyz/... but as if they were in /Game/xyz/... instead, for the context of asset restriction 
		// validations.
		TArray<FString> SplitPath;
		if (PackagePath.ParseIntoArray(SplitPath, TEXT("/")) > 1)
		{
			if ((SplitPath[1] == FPackagePath::GetExternalObjectsFolderName()) || (SplitPath[1] == FPackagePath::GetExternalActorsFolderName()))
			{
				SplitPath.RemoveAt(1);

				// Reconstruct path without any external objects paths
				TStringBuilder<1024> NewPackagePath;
				for (const FString& Path : SplitPath)
				{
					NewPackagePath += TEXT("/");
					NewPackagePath += Path;
				}
				PackagePath = NewPackagePath.ToString();
			}
		}

		DomainResult = PathMap->FindDomainFromPath(PackagePath);
	}

#if UE_ASSET_DOMAIN_FILTERING_DEBUG_LOGGING
	UE_LOGF(LogAssetReferenceRestrictions, Verbose, "Asset %ls belongs to domain %ls",
		*AssetData.ObjectPath.ToString(),
		DomainResult.IsValid() ? *DomainResult->UserFacingDomainName.ToString() : TEXT("(unknown)"));
#endif

	return DomainResult;
}

TTuple<bool, FText> FDomainDatabase::CanDomainsSeeEachOther(TSharedPtr<FDomainData> Referencee, TSharedPtr<FDomainData> Referencer) const
{
	check(Referencee.IsValid());
	check(Referencer.IsValid());

	if ((Referencee == Referencer) || Referencer->DomainsVisibleFromHere.Contains(Referencee) || Referencee->bCanBeSeenByEverything)
	{
		return MakeTuple(true, FText::GetEmpty());
	}
	else
	{
		FText ErrorText;

		if (Referencer->DomainsVisibleFromHere.Num() > 0)
		{
			const FString VisibleDomains = FString::JoinBy(Referencer->DomainsVisibleFromHere, TEXT(", "), [](TSharedPtr<FDomainData> Foo) { return Foo->UserFacingDomainName.ToString(); });
			ErrorText = FText::Format(LOCTEXT("InvalidReference_DomainList", "You may only reference assets from {0}, and {1} here"), FText::AsCultureInvariant(VisibleDomains), Referencer->UserFacingDomainName);
		}
		else
		{
			ErrorText = FText::Format(LOCTEXT("InvalidReference_NoVisibleDomains", "You may only reference assets from {0} here"), Referencer->UserFacingDomainName);
		}

		return MakeTuple(false, ErrorText);
	}
}

void FDomainDatabase::AddDomainVisibilityList(TSharedPtr<FDomainData> Domain, const TArray<FString>& VisibilityList)
{
	for (const FString& OtherDomainName : VisibilityList)
	{
		if (!OtherDomainName.IsEmpty())
		{
			Domain->DomainsVisibleFromHere.Add(FindOrAddDomainByName(OtherDomainName));
		}
	}
}

#undef LOCTEXT_NAMESPACE
