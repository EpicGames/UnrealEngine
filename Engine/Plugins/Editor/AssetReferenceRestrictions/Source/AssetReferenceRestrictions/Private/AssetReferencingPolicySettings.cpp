// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetReferencingPolicySettings.h"
#include "AssetReferencingDomains.h"
#include "AssetReferencingPolicySubsystem.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetReferencingPolicySettings)

const FString UAssetReferencingPolicySettings::EngineDomainName(TEXT("EngineContent"));
const FString UAssetReferencingPolicySettings::ScriptDomainName(TEXT("Script"));
const FString UAssetReferencingPolicySettings::GameDomainName(TEXT("ProjectContent"));
const FString UAssetReferencingPolicySettings::TempDomainName(TEXT("Temp"));
const FString UAssetReferencingPolicySettings::NeverCookDomainName(TEXT("NeverCook"));

UAssetReferencingPolicySettings::UAssetReferencingPolicySettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FARPDomainDefinitionForMatchingPlugins& GameFeatureRule = ProjectPlugins.AdditionalRules.AddDefaulted_GetRef();
	GameFeatureRule.DisplayName = NSLOCTEXT("AssetReferencingPolicy", "GameFeatureRule_DisplayName", "GameFeature:{0}");
	GameFeatureRule.ErrorMessageIfUsedElsewhere = NSLOCTEXT("AssetReferencingPolicy", "GameFeatureRule_ErrorMessageIfUsedElsewhere", "Game Feature content can only be accessed by other plugins that declare an explicit dependency");
	GameFeatureRule.MatchRule = EARPPluginMatchMode::MatchByPathPrefix;
	GameFeatureRule.PluginPathPrefix = TEXT("/GameFeatures/");
	GameFeatureRule.CanReferenceTheseDomains.Add(GameDomainName);

	EnginePlugins.DefaultRule.bCanBeSeenByOtherDomainsWithoutDependency = true;

	ProjectPlugins.DefaultRule.CanReferenceTheseDomains.Add(GameDomainName);
}

#if WITH_EDITOR
void UAssetReferencingPolicySettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (UAssetReferencingPolicySubsystem* Subsystem = GEditor->GetEditorSubsystem<UAssetReferencingPolicySubsystem>())
	{
		Subsystem->GetDomainDB()->MarkDirty();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

TArray<FString> UAssetReferencingPolicySettings::GetListOfDomains(bool bAllowPlugins, bool bAllowEngine, bool bAllowGame) const
{
	TArray<FString> Result;

	if (bAllowGame)
	{
		Result.Add(GameDomainName);
	}

	for (const FARPDomainDefinitionByContentRoot& DomainDef : AdditionalDomains)
	{
		Result.Add(DomainDef.DomainName);
	}

	if (bAllowPlugins)
	{
		if (UAssetReferencingPolicySubsystem* Subsystem = GEditor->GetEditorSubsystem<UAssetReferencingPolicySubsystem>())
		{
			Result.Append(Subsystem->GetDomainDB()->GetDomainsDefinedByPlugins());
		}
	}

	if (bAllowEngine)
	{
		Result.Add(EngineDomainName);
	}

	return Result;
}


