// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetReferencingPolicySubsystem.h"

#include "AssetReferencingDomains.h"
#include "AssetReferencingPolicySettings.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DomainAssetReferenceFilter.h"
#include "Editor.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/PathViews.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetReferencingPolicySubsystem)

#define LOCTEXT_NAMESPACE "AssetReferencingPolicy"

namespace AssetReferencingPolicySubsystem::Private
{
	bool bCheckForIllegalCppRefs = false;
	static FAutoConsoleVariableRef CVarCheckForIllegalCppRefs(
		TEXT("AssetReferencingPolicy.CheckForIllegalCppRefs"),
		bCheckForIllegalCppRefs,
		TEXT("Error on illegal references to C++ modules when validating asset references.")
	);

	bool bCheckForMissingHardRefs = false;
	static FAutoConsoleVariableRef CVarCheckForMissingHardRefs(
		TEXT("AssetReferencingPolicy.CheckForMissingHardRefs"),
		bCheckForMissingHardRefs,
		TEXT("Error on missing hard dependencies when validating asset references.")
	);

	bool bCheckForMissingRefsToExternalActors = false;
	static FAutoConsoleVariableRef CVarCheckForMissingRefsToExternalActors(
		TEXT("AssetReferencingPolicy.CheckForMissingRefsToExternalActors"),
		bCheckForMissingRefsToExternalActors,
		TEXT("Error on missing dependencies to external actors when validating asset references.")
	);
}

bool UAssetReferencingPolicySubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	return GetDefault<UAssetReferencingPolicySettings>()->bUseAssetReferenceRestrictions;
}

void UAssetReferencingPolicySubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	check(GEditor);
	GEditor->OnMakeAssetReferenceFilter().BindUObject(this, &ThisClass::HandleMakeAssetReferenceFilter);

	DomainDB = MakeShared<FDomainDatabase>();
	DomainDB->Init();

	FEditorDelegates::OnPreAssetValidation.AddUObject(this, &UAssetReferencingPolicySubsystem::UpdateDBIfNecessary);
}

void UAssetReferencingPolicySubsystem::Deinitialize()
{
	check(GEditor);
	GEditor->OnMakeAssetReferenceFilter().Unbind();
	DomainDB.Reset();
}

bool UAssetReferencingPolicySubsystem::ShouldValidateAssetReferences(const FAssetData& Asset) const
{
	TSharedPtr<FDomainData> DomainData = GetDomainDB()->FindDomainFromAssetData(Asset);

	const bool bIsInUnrestrictedFolder = DomainData && DomainData->IsValid() && DomainData->bCanSeeEverything;
	return !bIsInUnrestrictedFolder;
}

TValueOrError<void, TArray<FAssetReferenceError>> UAssetReferencingPolicySubsystem::ValidateAssetReferences(const FAssetData& InAssetData) const
{
	return ValidateAssetReferences(InAssetData, EAssetReferenceFilterRole::None);
}

TValueOrError<void, TArray<FAssetReferenceError>> UAssetReferencingPolicySubsystem::ValidateAssetReferences(const FAssetData& InAssetData, const EAssetReferenceFilterRole Role) const
{
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	TArray<FAssetReferenceError> Errors;
	static const FName TransientName = GetTransientPackage()->GetFName();

	// Check for missing soft or hard references to cinematic and developers content
	FName PackageFName = InAssetData.PackageName;
	TSet<FAssetData> AllDependencyAssets;

	const UAssetReferencingPolicySettings* Settings = GetDefault<UAssetReferencingPolicySettings>();
	UE::AssetRegistry::EDependencyQuery QueryFlags = UE::AssetRegistry::EDependencyQuery::NoRequirements;

	if (Settings->bIgnoreEditorOnlyReferences)
	{
		// If the rules allow ignoring editor-only referencers, restrict the dependencies to game references
		QueryFlags = UE::AssetRegistry::EDependencyQuery::Game;
	}

	auto ValidateDependencies = 
		[PackageFName, &InAssetData, &AssetRegistry, &AllDependencyAssets, &Errors](const UE::AssetRegistry::EDependencyQuery DependenciesQueryFlags, 
			const FText& MissingRefMsg, const FText& MissingRefSuffix, 
			const FText& UnsavedRefMsg, const FText& UnsavedRefSuffix, 
			const bool bReportErrors = true)
		{
			TArray<FName> PackageDependencies;
			AssetRegistry.GetDependencies(PackageFName, PackageDependencies, UE::AssetRegistry::EDependencyCategory::Package, DependenciesQueryFlags);
			for (FName PackageDependency : PackageDependencies)
			{
				FNameBuilder PackageDependencyStr(PackageDependency);
				if (FPackageName::IsScriptPackage(PackageDependencyStr))
				{
					if (AssetReferencingPolicySubsystem::Private::bCheckForIllegalCppRefs)
					{
						AllDependencyAssets.Add(FAssetData(PackageDependencyStr, PackageDependencyStr, UPackage::StaticClass()->GetClassPathName()));
					}
				}
				else if (FPackageName::IsVersePackage(PackageDependencyStr))
				{
					// Verse packages can be skipped for now. They will not exist on disk in some cases where the validator is run
				}
				else
				{
					TArray<FAssetData> DependencyAssets;
					AssetRegistry.GetAssetsByPackageName(PackageDependency, DependencyAssets, true);
					if (DependencyAssets.Num() == 0)
					{
						if (bReportErrors && PackageDependency != TransientName && !InAssetData.IsRedirector()) // Skip "does not exist" for redirectors as they may redirect to not-loaded plugins
						{
							FText ErrorMsg = MissingRefMsg;
							FText SuffixMsg = MissingRefSuffix;
							if (const UPackage* ExistingPackage = FindObjectFast<UPackage>(nullptr, PackageDependency);
								ExistingPackage && ExistingPackage->HasAnyPackageFlags(PKG_NewlyCreated))
							{
								ErrorMsg = UnsavedRefMsg;
								SuffixMsg = UnsavedRefSuffix;
							}
							
							Errors.Add(FAssetReferenceError{
								.Type = EAssetReferenceErrorType::DoesNotExist,
								.ReferencedAsset = FAssetData(PackageDependency, FName(FPathViews::GetPath(PackageDependencyStr)), FName(FPackageName::GetLongPackageAssetName(PackageDependencyStr.ToString())), UObject::StaticClass()->GetClassPathName()),
								.Message = MoveTemp(ErrorMsg),
								.Suffix = MoveTemp(SuffixMsg),
								});
						}
					}
					else
					{
						AllDependencyAssets.Append(DependencyAssets);
					}
				}
			}
		};

	ValidateDependencies(
		UE::AssetRegistry::EDependencyQuery::Soft | QueryFlags, 
		LOCTEXT("IllegalReference_MissingSoftRef", "soft references a missing package:"), FText(),
		LOCTEXT("IllegalReference_UnsavedSoftRef", "soft references an unsaved package:"), FText()
		);
	ValidateDependencies(
		UE::AssetRegistry::EDependencyQuery::Hard | QueryFlags, 
		LOCTEXT("IllegalReference_MissingHardRef", "hard references a missing package:"), LOCTEXT("IllegalReference_MissingHardRefSuffix", "(resave to clear this reference)"),
		LOCTEXT("IllegalReference_UnsavedHardRef", "hard references an unsaved package:"), FText(), 
		AssetReferencingPolicySubsystem::Private::bCheckForMissingHardRefs
		);

	// Check for missing external actor references which can manifest as LoadErrors and show as either Hard/Soft/Game/EditorOnly.
	if (AssetReferencingPolicySubsystem::Private::bCheckForMissingRefsToExternalActors)
	{
		TArray<FName> PkgDependencies;
		AssetRegistry.GetDependencies(PackageFName, PkgDependencies, UE::AssetRegistry::EDependencyCategory::Package);
		for (FName Dependency : PkgDependencies)
		{
			TArray<FAssetData> DependencyAssets;
			AssetRegistry.GetAssetsByPackageName(Dependency, DependencyAssets, true);
			
			FNameBuilder DependencyStr(Dependency);
			const bool bIsExternalActorPackage = DependencyStr.ToView().Contains(FPackagePath::GetExternalActorsFolderName());
			if (bIsExternalActorPackage && DependencyAssets.Num() == 0)
			{
				Errors.Add(FAssetReferenceError{
					.Type = EAssetReferenceErrorType::DoesNotExist,
					.ReferencedAsset = FAssetData(Dependency, FName(FPathViews::GetPath(DependencyStr)), FName(), UObject::StaticClass()->GetClassPathName()),
					.Message = LOCTEXT("IllegalReference_MissingRefToExternalActor", "references a missing external actor package:"),
				});
			}
		}
	}

	if (AllDependencyAssets.Num() > 0)
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;
		AssetReferenceFilterContext.AddRole(Role);
		AssetReferenceFilterContext.AddReferencingAsset(InAssetData);
		TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = GEditor ? GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext) : nullptr;
		if (ensure(AssetReferenceFilter.IsValid()))
		{
			IPluginManager& PluginManager = IPluginManager::Get();
			const bool bReferenceErrorsAsWarnings = AssetReferenceFilter->DoesAssetDowngradeReferenceErrorsToWarnings(InAssetData);
			for (const FAssetData& Dependency : AllDependencyAssets)
			{
				FText FailureReason;
				if (!AssetReferenceFilter->PassesFilter(Dependency, &FailureReason))
				{
					FText ErrorMsg;

					FNameBuilder DependencyStr(Dependency.PackageName);
					if (FPackageName::IsScriptPackage(DependencyStr))
					{
						const FName ReferencedModuleName(DependencyStr.ToView().RightChop(8));
						if (TSharedPtr<IPlugin> ReferencedPlugin = PluginManager.GetModuleOwnerPlugin(ReferencedModuleName))
						{
							ErrorMsg = FText::Format(
								LOCTEXT("IllegalReference_AssetFilterFailCpp", "illegally references C++ code in plugin '{0}' (module package: '{1}') via:"),
								FText::FromString(ReferencedPlugin->GetName()), FText::FromStringView(DependencyStr)
							);
						}
					}

					if (ErrorMsg.IsEmpty())
					{
						ErrorMsg = LOCTEXT("IllegalReference_AssetFilterFail", "illegally references:");
					}

					FText SuffixMsg;
					if (!FailureReason.IsEmpty())
					{
						SuffixMsg = FText::Format(LOCTEXT("IllegalReference_SuffixFmt", "- {0}"), FailureReason);
					}

					Errors.Add(FAssetReferenceError{
						.bTreatErrorAsWarning = bReferenceErrorsAsWarnings,
						.Type = EAssetReferenceErrorType::Illegal,
						.ReferencedAsset = Dependency,
						.Message = MoveTemp(ErrorMsg),
						.Suffix = MoveTemp(SuffixMsg),
					});
				}
			}
		}
	}
	
	if (Errors.Num())
	{
		return MakeError(MoveTemp(Errors));
	}
	else
	{
		return MakeValue();
	}
}

TSharedPtr<IAssetReferenceFilter> UAssetReferencingPolicySubsystem::HandleMakeAssetReferenceFilter(const FAssetReferenceFilterContext& Context)
{
	return MakeShareable(new FDomainAssetReferenceFilter(Context, GetDomainDB()));
}

void UAssetReferencingPolicySubsystem::UpdateDBIfNecessary() const
{
	DomainDB->UpdateIfNecessary();
}

TSharedPtr<FDomainDatabase> UAssetReferencingPolicySubsystem::GetDomainDB() const
{
	DomainDB->UpdateIfNecessary();
	return DomainDB;
}

FAutoConsoleCommandWithWorldAndArgs GListDomainDatabaseCmd(
	TEXT("Editor.AssetReferenceRestrictions.ListDomainDatabase"),
	TEXT("Lists all of the asset reference domains the AssetReferenceRestrictions plugin knows about"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(
		[](const TArray<FString>& Params, UWorld* World)
		{
			if (UAssetReferencingPolicySubsystem* Subsystem = GEditor->GetEditorSubsystem<UAssetReferencingPolicySubsystem>())
			{
				Subsystem->GetDomainDB()->DebugPrintAllDomains();
			}
		}));

#undef LOCTEXT_NAMESPACE
