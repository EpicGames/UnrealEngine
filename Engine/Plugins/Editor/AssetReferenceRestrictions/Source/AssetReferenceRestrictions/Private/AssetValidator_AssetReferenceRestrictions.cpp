// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetValidator_AssetReferenceRestrictions.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataToken.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor.h"
#include "Editor/AssetReferenceFixer.h"
#include "AssetReferencingPolicySubsystem.h"
#include "AssetReferencingPolicySettings.h"
#include "AssetReferencingDomains.h"
#include "Editor/AssetReferenceFilter.h"
#include "Misc/PackageName.h"
#include "Misc/DataValidation.h"
#include "Misc/DataValidation/Fixer.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetValidator_AssetReferenceRestrictions)

#define LOCTEXT_NAMESPACE "AssetReferencingPolicy"

UAssetValidator_AssetReferenceRestrictions::UAssetValidator_AssetReferenceRestrictions()
	: Super()
{
}

bool UAssetValidator_AssetReferenceRestrictions::CanValidateAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext) const
{
	if (InAsset)
	{
		if (UAssetReferencingPolicySubsystem* Subsystem = GEditor->GetEditorSubsystem<UAssetReferencingPolicySubsystem>())
		{
			return Subsystem->ShouldValidateAssetReferences(AssetData);
		}
	}

	return false;
}

EDataValidationResult UAssetValidator_AssetReferenceRestrictions::ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext)
{
	check(InAsset);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	const IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	// Validate asset references
	ValidateAssetInternal(InAssetData, AssetRegistry);
	
	// Validate each external object's references
	for (const FAssetData& ExternalObject : InContext.GetAssociatedExternalObjects())
	{
		ValidateAssetInternal(ExternalObject, AssetRegistry);
	}
	
	if (GetValidationResult() != EDataValidationResult::Invalid)
	{
		AssetPasses(InAsset);
	}

	return GetValidationResult();
}

void UAssetValidator_AssetReferenceRestrictions::ValidateAssetInternal(const FAssetData& InAssetData, const IAssetRegistry& InAssetRegistry)
{
    UAssetReferencingPolicySubsystem* Subsystem = GEditor->GetEditorSubsystem<UAssetReferencingPolicySubsystem>();
	check(Subsystem);

	if (TValueOrError<void, TArray<FAssetReferenceError>> Result = Subsystem->ValidateAssetReferences(InAssetData, EAssetReferenceFilterRole::Validation);
		Result.HasError())
	{
		TSharedPtr<IAssetReferenceFixer> AssetReferenceFixer;
		for (const FAssetReferenceError& Error : Result.GetError())
		{
			TSharedRef<FTokenizedMessage> TokenizedMessage = AssetMessage(InAssetData, Error.bTreatErrorAsWarning ? EMessageSeverity::Warning : EMessageSeverity::Error, Error.Message)
				->AddToken(FAssetDataToken::Create(Error.ReferencedAsset));
			if (!Error.Suffix.IsEmpty())
			{
				TokenizedMessage->AddToken(FTextToken::Create(Error.Suffix));
			}
			TokenizedMessage->AddToken(FTextToken::Create(FText::FormatNamed(LOCTEXT("ValidatorClassSuffix", ". ({ValidatorName})"), TEXT("ValidatorName"), FText::AsCultureInvariant(GetClass()->GetName()))));

			if (Error.Type == EAssetReferenceErrorType::Illegal)
			{
				if (!AssetReferenceFixer)
				{
					AssetReferenceFixer = GEditor->MakeAssetReferenceFixer();
				}
				if (AssetReferenceFixer)
				{
					if (TSharedPtr<UE::DataValidation::IFixer> Fixer = AssetReferenceFixer->CreateFixer(InAssetData))
					{
						TokenizedMessage->AddToken(Fixer->CreateToken(AssetReferenceFixer->GetFixerLabel(InAssetData)));
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
