// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorValidatorBase.h"
#include "AssetValidator_AssetReferenceRestrictions.generated.h"

class IAssetRegistry;

UCLASS()
class UAssetValidator_AssetReferenceRestrictions : public UEditorValidatorBase
{
	GENERATED_BODY()

public:
	UAssetValidator_AssetReferenceRestrictions();

protected:
	//~UEditorValidatorBase interface
	virtual bool CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) const override;
	virtual EDataValidationResult ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) override;
	//~End of UEditorValidatorBase interface

private:
	void ValidateAssetInternal(const FAssetData& InAssetData, const IAssetRegistry& InAssetRegistry);
};
