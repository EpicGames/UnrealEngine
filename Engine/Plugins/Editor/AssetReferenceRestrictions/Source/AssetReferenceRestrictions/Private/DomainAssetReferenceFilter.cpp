// Copyright Epic Games, Inc. All Rights Reserved.

#include "DomainAssetReferenceFilter.h"
#include "Editor/EditorEngine.h"
#include "Toolkits/ToolkitManager.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/IToolkit.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetReferencingDomains.h"

#define LOCTEXT_NAMESPACE "AssetReferencingPolicy"

TArray<FDomainAssetReferenceFilter*> FDomainAssetReferenceFilter::FilterInstances;

FDomainAssetReferenceFilter::FDomainAssetReferenceFilter(const FAssetReferenceFilterContext& Context, TSharedPtr<FDomainDatabase> InDomainDB)
	: IAssetReferenceFilter()
	, DomainDB(InDomainDB)
{
	OriginalReferencingAssets = Context.GetReferencingAssets();

	DetermineReferencingDomain();

	Failure_CouldNotDetermineDomain = LOCTEXT("Failure_CouldNotDetermineDomain", "Could not find a matching domain for {0}");
	
	FilterInstances.Add(this);
}

FDomainAssetReferenceFilter::~FDomainAssetReferenceFilter()
{
	FilterInstances.Remove(this);
}

void FDomainAssetReferenceFilter::DetermineReferencingDomain()
{
	QUICK_SCOPE_CYCLE_COUNTER(FDomainAssetReferenceFilter_DetermineReferencingDomain);

	ReferencingAssetDataInfos.Reset();
	
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	// Collect the associated domains from each referencing asset
	TArray<FAssetData> PendingAssets;
	TSet<FAssetData> AssetsConsidered;

	for (const FAssetReferenceFilterReferencerInfo& AssetReferencer : OriginalReferencingAssets)
	{
		PendingAssets.Add(AssetReferencer.Data);
	}

	while (!PendingAssets.IsEmpty())
	{
		const FAssetData TestAsset = PendingAssets.Pop(EAllowShrinking::No);

		if (AssetsConsidered.Contains(TestAsset))
		{
			continue;
		}
		AssetsConsidered.Add(TestAsset);

		// Skip redirectors if they are themselves unreferenced
		if (TestAsset.IsRedirector())
		{
			TArray<FName> RedirectorRefs;
			AssetRegistryModule.Get().GetReferencers(TestAsset.PackageName, RedirectorRefs);
			if (RedirectorRefs.Num() == 0)
			{
				continue;
			}
		} 

		// Objects in the transient package don't naturally have any domain, but if they're preview objects
		// for an asset editor then we want to consider that asset as well
		UObject* LoadedObj = TestAsset.IsAssetLoaded() ? TestAsset.GetAsset() : nullptr;

		if ((LoadedObj != nullptr) && (LoadedObj->GetOutermost() == GetTransientPackage()))
		{
			TryGetAssociatedAssetsFromPossiblyPreviewObject(LoadedObj, /*inout*/ PendingAssets);
			continue;
		}

		// Determine the domain from the asset
		TSharedPtr<FDomainData> TestDomain = DomainDB->FindDomainFromAssetData(TestAsset);
		if (TestDomain.IsValid())
		{
			ReferencingAssetDataInfos.Add(MakeTuple(TestAsset, TestDomain));
		}
	}
}

bool FDomainAssetReferenceFilter::PassesFilter(const FAssetData& AssetData, FText* OutOptionalFailureReason) const
{
	FText MyReason;
	bool bMyResult = PassesFilterImpl(AssetData, /*out*/ MyReason);

	if (OutOptionalFailureReason)
	{
		*OutOptionalFailureReason = MyReason;
	}
	return bMyResult;
}

void FDomainAssetReferenceFilter::UpdateAllFilters()
{
	for (FDomainAssetReferenceFilter* Instance : FilterInstances)
	{
		Instance->DetermineReferencingDomain();
	}
}

bool FDomainAssetReferenceFilter::IsCrossPluginReferenceAllowed(const FAssetDataInfo& ReferencingAssetDataInfo, const FAssetDataInfo& ReferencedAssetDataInfo) const
{
	const TSharedPtr<FDomainData>& ReferencingAssetDomain = ReferencingAssetDataInfo.Value;
	const TSharedPtr<FDomainData>& ReferencedAssetDomain = ReferencedAssetDataInfo.Value;
	const FAssetData& ReferencingAssetData = ReferencingAssetDataInfo.Key;
	const FAssetData& ReferencedAssetData = ReferencedAssetDataInfo.Key;

	return DomainDB->IsPlugin(ReferencedAssetDomain) &&
		   DomainDB->IsPlugin(ReferencingAssetDomain) &&
		   IAssetReferenceFilter::IsCrossPluginReferenceAllowed(ReferencingAssetData, ReferencedAssetData);
}

bool FDomainAssetReferenceFilter::PassesFilterImpl(const FAssetData& AssetData, FText& OutOptionalFailureReason) const
{
	QUICK_SCOPE_CYCLE_COUNTER(FDomainAssetReferenceFilter_PassesFilterImpl);

	// An empty/null asset reference is always valid (e.g., the (none) option in the class picker)
	if (!AssetData.IsValid())
	{
		return true;
	}

	TSharedPtr<FDomainData> AssetDomain = DomainDB->FindDomainFromAssetData(AssetData);

	// This ensure should never fail; every valid asset should belong to engine, special system mount,
	// game, or a plugin that is allowed to contain content, and we create domains for all of those.
	// The only exception anticipated is if somehow an editor with an asset picker is pointed
	// towards something living in the transient package, but that's not expected!
	// If it gets hit, transient can be added to the special system mount 'Temp' domain
	if (ensureMsgf(AssetDomain.IsValid(), TEXT("Asset %s didn't match any domain"), *AssetData.GetObjectPathString()))
	{
		FAssetDataInfo ReferencedAssetDataInfo = MakeTuple(AssetData, AssetDomain);

		// Check the referencing domains
		for (const FAssetDataInfo& ReferencingAssetDataInfo : ReferencingAssetDataInfos)
		{
			const TSharedPtr<FDomainData>& ReferencingDomain = ReferencingAssetDataInfo.Value;
			if (ReferencingDomain->bCanSeeEverything)
			{
				return true;
			}

			bool bResult;
			Tie(bResult, OutOptionalFailureReason) = DomainDB->CanDomainsSeeEachOther(AssetDomain, ReferencingDomain);

			if (!bResult)
			{
				// Before returning false, check whether this is an allowed cross-plugin asset reference (exception)
				if (IsCrossPluginReferenceAllowed(ReferencingAssetDataInfo, ReferencedAssetDataInfo))
				{
					return true;
				}
				
				return false;
			}
		}

		// None of the referencing domains said no, so we can see it
		return true;
	}
	else
	{
		OutOptionalFailureReason = FText::Format(Failure_CouldNotDetermineDomain, FText::FromName(AssetData.PackageName));
		return false;
	}
}

void FDomainAssetReferenceFilter::TryGetAssociatedAssetsFromPossiblyPreviewObject(UObject* PossiblyPreviewObject, TArray<FAssetData>& InOutAssetsToConsider) const
{
	UObject* Obj = PossiblyPreviewObject;

	// Find the object within the outermost upackage
	while (Obj && Obj->GetOuter() && !Obj->GetOuter()->IsA(UPackage::StaticClass()))
	{
		Obj = Obj->GetOuter();
	}

	if (Obj)
	{
		TSharedPtr<IToolkit> FoundToolkit = FToolkitManager::Get().FindEditorForAsset(Obj);
		if (FoundToolkit.IsValid())
		{
			if (const TArray<UObject*>* EditedObjects = FoundToolkit->GetObjectsCurrentlyBeingEdited())
			{
				for (UObject* EditedObject : *EditedObjects)
				{
					if ((EditedObject != Obj) && (EditedObject->GetOutermost() != GetTransientPackage()))
					{
						// Found an asset from this toolkit that was not the preview object, use it instead.
						InOutAssetsToConsider.Emplace(EditedObject);
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
