// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorValidator_PluginAssetReferences.h"

#include "Algo/RemoveIf.h"
#include "AssetReferencingDomains.h"
#include "AssetReferencingPolicySettings.h"
#include "AssetReferencingPolicySubsystem.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataToken.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DataValidationChangelist.h"
#include "Editor.h"
#include "EditorValidatorSubsystem.h"
#include "Editor/EditorEngine.h"
#include "Interfaces/IPluginManager.h"
#include "Logging/StructuredLog.h"
#include "Misc/PathViews.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorValidator_PluginAssetReferences)

#define LOCTEXT_NAMESPACE "AssetReferencingPolicy"

bool UEditorValidator_PluginAssetReferences::CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InObject, FDataValidationContext& InContext) const
{
	if (!GEditor->GetEditorSubsystem<UAssetReferencingPolicySubsystem>())
	{
		// AssetReferenceRestrictions is disabled
		return false;
	}

    if (UDataValidationChangelist* Changelist = Cast<UDataValidationChangelist>(InObject))
    {
        for (const FString& ModifiedFilePath : Changelist->ModifiedFiles)
        {
            if (FPathViews::GetExtension(ModifiedFilePath) == TEXTVIEW("uplugin"))
            {
                return true;
            }
        }
    }
    return false;
}

EDataValidationResult UEditorValidator_PluginAssetReferences::ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& Context) 
{
    UDataValidationChangelist* Changelist = Cast<UDataValidationChangelist>(InAsset);
    if (!Changelist)
    {
        return EDataValidationResult::Valid;
    }

    IPluginManager& PluginManager = IPluginManager::Get();
    TArray<TSharedRef<IPlugin>> PluginsToValidate;
    for (const FString& ModifiedFilePath : Changelist->ModifiedFiles)
    {
        if (FPathViews::GetExtension(ModifiedFilePath) == TEXTVIEW("uplugin"))
        {
            FStringView PluginName = FPathViews::GetBaseFilename(ModifiedFilePath);
            TSharedPtr<IPlugin> Plugin = PluginManager.FindPlugin(PluginName);
            if (Plugin.IsValid() && Plugin->IsEnabled())
            {
                UE_LOGFMT(LogAssetReferenceRestrictions, Display, "Validating asset references from plugin: {PluginName}", Plugin->GetName());
                PluginsToValidate.Add(Plugin.ToSharedRef());
            }
        }
    }

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
    TArray<FAssetData> AssetsToValidate;
    for (const TSharedRef<IPlugin>& Plugin : PluginsToValidate)
    {
        TStringBuilder<256> RootPath(InPlace, TEXT("/"), Plugin->GetName());
        AssetRegistry.GetAssetsByPath(FName(RootPath), AssetsToValidate, true, true);
    }

    UAssetReferencingPolicySubsystem* AssetReferencingPolicySubsystem = GEditor->GetEditorSubsystem<UAssetReferencingPolicySubsystem>();
	UEditorValidatorSubsystem* EditorValidationSubsystem = GEditor->GetEditorSubsystem<UEditorValidatorSubsystem>();
	FValidateAssetsSettings Settings;

    auto FilterAssetsToValidate = [&Settings, AssetReferencingPolicySubsystem, EditorValidationSubsystem](TArray<FAssetData>& InAssets){
        FDataValidationContext ValidationContext(false, Settings.ValidationUsecase, {});
        InAssets.SetNum(Algo::RemoveIf(InAssets, [AssetReferencingPolicySubsystem, EditorValidationSubsystem, &Settings, &ValidationContext](const FAssetData& Asset) {
            return !AssetReferencingPolicySubsystem->ShouldValidateAssetReferences(Asset) || !EditorValidationSubsystem->ShouldValidateAsset(Asset, Settings, ValidationContext);
        }));
    };

    FilterAssetsToValidate(AssetsToValidate);

	const UAssetReferencingPolicySettings* PolicySettings = GetDefault<UAssetReferencingPolicySettings>();
	UE::AssetRegistry::EDependencyQuery QueryFlags = UE::AssetRegistry::EDependencyQuery::NoRequirements;
	if (PolicySettings->bIgnoreEditorOnlyReferences)
	{
		// If the rules allow ignoring editor-only referencers, restrict the dependencies to game references
		QueryFlags = UE::AssetRegistry::EDependencyQuery::Game;
	}

    for (const FAssetData& Asset : AssetsToValidate)
    {
        TValueOrError<void, TArray<FAssetReferenceError>> Result = AssetReferencingPolicySubsystem->ValidateAssetReferences(Asset);
        if (Result.HasError())
        {
            for (const FAssetReferenceError& Error : Result.GetError())
            {
				TSharedRef<FTokenizedMessage> Msg = AssetMessage(Asset, Error.bTreatErrorAsWarning ? EMessageSeverity::Warning : EMessageSeverity::Error, Error.Message)
					->AddToken(FAssetDataToken::Create(Error.ReferencedAsset));
				if (!Error.Suffix.IsEmpty())
				{
					Msg->AddToken(FTextToken::Create(Error.Suffix));
				}
                if (Asset.IsRedirector())
                {
                    TArray<FAssetData> RedirectorReferencers;
                    TArray<FName> RedirectorReferencerPackages;
                    AssetRegistry.GetReferencers(Asset.PackageName, RedirectorReferencerPackages, UE::AssetRegistry::EDependencyCategory::Package, QueryFlags);
                    for (FName PackageName : RedirectorReferencerPackages)
                    {
                        AssetRegistry.GetAssetsByPackageName(PackageName, RedirectorReferencers);
                    }
                    FilterAssetsToValidate(RedirectorReferencers);

                    if (RedirectorReferencers.Num())
                    {
                        Msg->AddText(LOCTEXT("InvalidReferences_ReferencedRedirector", "Note that the referencing asset is a redirector which is referenced by the following assets: "));
                        for (const FAssetData& Referencer : RedirectorReferencers)
                        {
                            Msg->AddToken(FAssetDataToken::Create(Referencer));
                        }
                    }
                    else
                    {
                        Msg->AddText(LOCTEXT("InvalidReferences_UnreferencedRedirector", "Note that the referencing asset is a redirector which is not referenced by anything else."));
                    }
                }
            }
        }
    }

    if(GetValidationResult() != EDataValidationResult::Invalid)
    {
        AssetPasses(InAsset);
    }
    return GetValidationResult();
}

#undef LOCTEXT_NAMESPACE 