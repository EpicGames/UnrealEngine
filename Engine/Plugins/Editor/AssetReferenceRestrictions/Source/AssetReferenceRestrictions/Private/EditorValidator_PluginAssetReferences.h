// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorValidatorBase.h"

#include "EditorValidator_PluginAssetReferences.generated.h"

UCLASS()
class UEditorValidator_PluginAssetReferences : public UEditorValidatorBase
{
    GENERATED_BODY()

public:
	virtual bool CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InObject, FDataValidationContext& InContext) const override;
	virtual EDataValidationResult ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& Context) override;
};