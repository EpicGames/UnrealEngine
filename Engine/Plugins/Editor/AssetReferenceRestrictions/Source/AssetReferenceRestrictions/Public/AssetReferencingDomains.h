// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"

#define UE_API ASSETREFERENCERESTRICTIONS_API

//@TODO: Debug printing
#define UE_ASSET_DOMAIN_FILTERING_DEBUG_LOGGING 0

#if UE_ASSET_DOMAIN_FILTERING_DEBUG_LOGGING
DECLARE_LOG_CATEGORY_EXTERN(LogAssetReferenceRestrictions, Verbose, All);
#else
DECLARE_LOG_CATEGORY_EXTERN(LogAssetReferenceRestrictions, Log, Log);
#endif

class IPlugin;
struct FDomainPathNode;
struct FAssetData;

struct FDomainData final : public TSharedFromThis<FDomainData>
{
	FText UserFacingDomainName;
	FText ErrorMessageIfUsedElsewhere;

	// The list of root paths, always of the format /Mount/ or /Mount/Path/To/ with both leading and trailing /
	TArray<FString> DomainRootPaths;

	// A list of specific packages that are part of this domain
	TArray<FName> SpecificAssetPackages;

	// The domains that are visible from here (if bCanSeeEverything is true, then literally everything is visible from here)
	TSet<TSharedPtr<FDomainData>> DomainsVisibleFromHere;

	// Can we see everything?
	bool bCanSeeEverything = false;

	// Can we be seen by everything?
	bool bCanBeSeenByEverything = false;

	bool IsValid() const
	{
		return DomainRootPaths.Num() > 0 || SpecificAssetPackages.Num() > 0;
	}

	void Reset()
	{
		UserFacingDomainName = FText::GetEmpty();
		DomainRootPaths.Reset();
		DomainsVisibleFromHere.Reset();
		bCanSeeEverything = false;
	}
};

struct FDomainDatabase final
{
	UE_API FDomainDatabase();
	UE_API ~FDomainDatabase();

	UE_API void Init();

	UE_API void MarkDirty();
	UE_API void UpdateIfNecessary();

	// Delegate that's called whenever the database has been updated to allow people to respond to the change
	TMulticastDelegate<void()>& PostDatabaseUpdated() { return OnPostDatabaseUpdated; };

	UE_API void ValidateAllDomains();
	UE_API void DebugPrintAllDomains();

	UE_API void OnPluginCreatedOrMounted(IPlugin& NewPlugin);

	UE_API TSharedPtr<FDomainData> FindDomainFromAssetData(const FAssetData& AssetData) const;

	UE_API TTuple<bool, FText> CanDomainsSeeEachOther(TSharedPtr<FDomainData> Referencee, TSharedPtr<FDomainData> Referencer) const;

	const TArray<FString>& GetDomainsDefinedByPlugins() const { return DomainsDefinedByPlugins; }
	bool IsPlugin(TSharedPtr<FDomainData> Domain) const { return PluginDomains.Contains(Domain); }

	UE_API TSharedPtr<FDomainData> FindOrAddDomainByName(const FString& Name);

private:
	UE_API void RebuildFromScratch();
	
	UE_API void BuildDomainFromPlugin(TSharedRef<IPlugin> Plugin);
	UE_API void BuildUnrestrictedDomainFromPlugin(TSharedRef<IPlugin> Plugin);

	UE_API void AddDomainVisibilityList(TSharedPtr<FDomainData> Domain, const TArray<FString>& VisibilityList);
private:
	// Map from domain name to 
	TMap<FString, TSharedPtr<FDomainData>> DomainNameMap;

	// Map from path to domain
	TSharedPtr<FDomainPathNode> PathMap;

	// Specific packages that are forced into certain domains
	TMap<FName, TSharedPtr<FDomainData>> SpecificAssetPackageDomains;

	// The engine content domain
	TSharedPtr<FDomainData> EngineDomain;

	// The default script domain (for reflected types)
	TSharedPtr<FDomainData> ScriptDomain;

	// Used for various 'special' mount points like /Temp/, /Memory/, and /Extra/
	// Not visible as a domain for other domains to see, and can see everything
	TSharedPtr<FDomainData> TempDomain;

	// The game content domain
	TSharedPtr<FDomainData> GameDomain;

	// The never cooked content domain
	TSharedPtr<FDomainData> NeverCookDomain;

	// List of domains that came from plugins (used for domain pickers in the settings)
	TArray<FString> DomainsDefinedByPlugins;

	// Plugin Domains
	TSet<TSharedPtr<FDomainData>> PluginDomains;

	bool bDatabaseOutOfDate = false;

	TMulticastDelegate<void()> OnPostDatabaseUpdated;
};

#undef UE_API
