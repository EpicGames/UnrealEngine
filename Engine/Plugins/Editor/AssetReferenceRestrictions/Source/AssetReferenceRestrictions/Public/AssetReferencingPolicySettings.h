// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "AssetReferencingPolicySettings.generated.h"

#define UE_API ASSETREFERENCERESTRICTIONS_API

struct FDirectoryPath;
struct FPropertyChangedEvent;

USTRUCT()
struct FARPDefaultPluginDomainRules
{
	GENERATED_BODY()

	// The list of additional domains always visible from a plugin
	// (EngineContent is always visible, as is content from other plugins that are explicitly referenced)
	UPROPERTY(EditAnywhere, Category=Settings, meta=(GetOptions="GetListOfDomains_NoPluginsOrEngine"))
	TArray<FString> CanReferenceTheseDomains;

	// Can content in the ProjectContent domain access plugin content automatically (for plugins that don't match a specific rule)?
	UPROPERTY(config, EditAnywhere, Category=Settings)
	bool bCanProjectAccessThesePlugins = true;

	// Can content in other domains access plugin content automatically (for plugins that don't match a specific rule)?
	// Note: This rule may be deprecated in the future!
	UPROPERTY(config, EditAnywhere, Category=Settings)
	bool bCanBeSeenByOtherDomainsWithoutDependency = false;
};

UENUM()
enum class EARPPluginMatchMode : uint8
{
	MatchByCategory,
	MatchByPathPrefix
};

USTRUCT()
struct FARPDomainDefinitionForMatchingPlugins
{
	GENERATED_BODY()

	// The display name of this rule (used in error message when attempting to reference content incorrectly)
	// The token {0} will be replaced with the plugin name
	UPROPERTY(EditAnywhere, Category=Settings)
	FText DisplayName;

	// The error message if something that is not allowed to attempts to reference content from this domain
	UPROPERTY(EditAnywhere, Category=Settings)
	FText ErrorMessageIfUsedElsewhere;

	// Type of matching for this rule
	UPROPERTY(EditAnywhere, Category=Settings)
	EARPPluginMatchMode MatchRule = EARPPluginMatchMode::MatchByPathPrefix;

	// If set, a plugin with the same rooted directory path will match this rule
	// (use "/FirstFolder/SecondFolder/" to match a plugin like $YourProjectDir/Plugins/FirstFolder/SecondFolder/MyCoolPlugin/MyCoolPlugin.uplugin)
	UPROPERTY(EditAnywhere, Category=Settings, meta=(EditCondition="MatchRule==EARPPluginMatchMode::MatchByPathPrefix", EditConditionHides))
	FString PluginPathPrefix;

	// If set, a plugin with a matching Category will match this rule
	UPROPERTY(EditAnywhere, Category=Settings, meta=(EditCondition="MatchRule==EARPPluginMatchMode::MatchByCategory", EditConditionHides))
	FString PluginCategoryPrefix;

	// The list of additional domains always visible from a plugin
	// (EngineContent is always visible, as is content from other plugins that are explicitly referenced)
	UPROPERTY(EditAnywhere, Category=Settings, meta=(GetOptions="GetListOfDomains_NoPluginsOrEngine"))
	TArray<FString> CanReferenceTheseDomains;

	bool IsValid() const
	{
		if (DisplayName.IsEmpty()) { return false; }

		if (MatchRule == EARPPluginMatchMode::MatchByCategory)
		{
			if (PluginCategoryPrefix.IsEmpty()) { return false; }
		}
		else
		{
			if (PluginPathPrefix.IsEmpty()) { return false; }
		}

		return true;
	}
};

USTRUCT()
struct FARPDomainSettingsForPlugins
{
	GENERATED_BODY()

	// The default rule if a more specific plugin rule doesn't apply
	UPROPERTY(EditAnywhere, Category=Settings)
	FARPDefaultPluginDomainRules DefaultRule;

	// Discovered plugins will be matched against these templates
	// Priority rules (a path match is preferred to a category match, and within each the longest match wins):
	//   Highest: The most specific path match
	//            Any path match
	//            The most specific category match
	//            Any category match
	UPROPERTY(EditAnywhere, Category=Settings, meta=(TitleProperty="DisplayName"))
	TArray<FARPDomainDefinitionForMatchingPlugins> AdditionalRules;
};


USTRUCT()
struct FARPDefaultProjectDomainRules
{
	GENERATED_BODY()

	/**
	 * The list of additional domains always visible from this domain.
	 * EngineContent and Project Content in the '/Game/' directory are always visible
	 */
	UPROPERTY(EditAnywhere, Category = Settings, meta=(GetOptions="GetListOfDomains_NoEngineOrGame"))
	TArray<FString> CanReferenceTheseDomains;
};


UENUM()
enum class EARPDomainAllowedToReferenceMode : uint8
{
	AdditionalDomains,
	AllDomains UMETA(DisplayName = "All Domains (DANGER)")
};


USTRUCT()
struct FARPDomainDefinitionByContentRoot
{
	GENERATED_BODY()

	// The name of this domain
	UPROPERTY(EditAnywhere, Category=Settings)
	FString DomainName;

	// The display name of this domain (used in error message when attempting to reference content incorrectly)
	UPROPERTY(EditAnywhere, Category=Settings)
	FText DomainDisplayName;

	// The error message if something that is not allowed to attempts to reference content from this domain
	UPROPERTY(EditAnywhere, Category=Settings)
	FText ErrorMessageIfUsedElsewhere;

	// The list of content root paths considered to be part of this domain
	UPROPERTY(EditAnywhere, Category=Settings, meta=(ContentDir))
	TArray<FDirectoryPath> ContentRoots;

	// A list of specific assets considered to be part of this domain
	UPROPERTY(EditAnywhere, Category=Settings, meta=(LongPackageName))
	TArray<FName> SpecificAssets;

	// What content is this domain allowed to access?
	UPROPERTY(EditAnywhere, Category = Settings)
	EARPDomainAllowedToReferenceMode ReferenceMode = EARPDomainAllowedToReferenceMode::AdditionalDomains;

	// The list of additional domains always visible from this domain
	// (EngineContent is always visible)
	UPROPERTY(EditAnywhere, Category = Settings, meta=(GetOptions="GetListOfDomains_NoEngine", EditCondition="ReferenceMode==EARPDomainAllowedToReferenceMode::AdditionalDomains", EditConditionHides))
	TArray<FString> CanReferenceTheseDomains;

	bool IsValid() const
	{
		if (DomainName.IsEmpty()) { return false; }

		if (ContentRoots.Num() == 0 && SpecificAssets.Num() == 0) { return false; }

		if (DomainDisplayName.IsEmpty()) { return false; }

		return true;
	}
};

/** Settings for the Asset Referencing Policy, these settings are used to determine which plugins and game folders can reference content from each other */
UCLASS(MinimalAPI, config=Game, defaultconfig, meta=(DisplayName="Asset Referencing Policy"))
class UAssetReferencingPolicySettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	// High level switch to enable or disable all AssetReferenceRestrictions
	UPROPERTY(config, EditAnywhere, Category = Settings)
	bool bUseAssetReferenceRestrictions = true;

	// Settings/rules for engine plugins
	UPROPERTY(config, EditAnywhere, Category="Engine Plugins")
	FARPDomainSettingsForPlugins EnginePlugins;

	// Settings/rules for project plugins
	UPROPERTY(config, EditAnywhere, Category="Project Plugins")
	FARPDomainSettingsForPlugins ProjectPlugins;

	// The default rules for project content (if a more specific rule doesn't apply)
	UPROPERTY(config, EditAnywhere, Category="Project Content")
	FARPDefaultProjectDomainRules DefaultProjectContentRule;

	// List of additional domains to carve out from the project content folder
	UPROPERTY(config, EditAnywhere, Category="Project Content", meta=(TitleProperty=DomainName))
	TArray<FARPDomainDefinitionByContentRoot> AdditionalDomains;

	// If true, ignore any editor-only domain references that will not affect the cooked game
	UPROPERTY(config, EditAnywhere, Category = Settings)
	bool bIgnoreEditorOnlyReferences = true;

	// The names of the project, special system mount, and game domains
	static UE_API const FString EngineDomainName;
	static UE_API const FString ScriptDomainName;
	static UE_API const FString GameDomainName;
	static UE_API const FString TempDomainName;
	static UE_API const FString NeverCookDomainName;

public:
	//~UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~End of UObject interface

	UE_API UAssetReferencingPolicySettings(const FObjectInitializer& ObjectInitializer);

private:
	UE_API TArray<FString> GetListOfDomains(bool bAllowPlugins, bool bAllowEngine, bool bAllowGame) const;

	UFUNCTION()
	TArray<FString> GetListOfDomains_All() const
	{
		return GetListOfDomains(true, true, true);
	}

	UFUNCTION()
	TArray<FString> GetListOfDomains_NoEngine() const
	{
		return GetListOfDomains(true, false, true);
	}

	UFUNCTION()
	TArray<FString> GetListOfDomains_NoEngineOrGame() const
	{
		return GetListOfDomains(true, false, false);
	}

	UFUNCTION()
	TArray<FString> GetListOfDomains_NoPluginsOrEngine() const
	{
		return GetListOfDomains(false, false, true);
	}

};

#undef UE_API
