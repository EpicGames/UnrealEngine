// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "EditorSubsystem.h"
#include "Templates/ValueOrError.h"

#include "AssetReferencingPolicySubsystem.generated.h"

#define UE_API ASSETREFERENCERESTRICTIONS_API

enum class EAssetReferenceFilterRole : uint8;
struct FAssetReferenceFilterContext;
class IAssetReferenceFilter;
struct FAssetData;
struct FDomainDatabase;

enum class EAssetReferenceErrorType
{
	DoesNotExist,
	Illegal
};

struct FAssetReferenceError
{
	bool bTreatErrorAsWarning = false;
	EAssetReferenceErrorType Type;
	FAssetData ReferencedAsset;
	FText Message;
	FText Suffix;
};

/** Subsystem to register the domain-based asset referencing policy restrictions with the editor */
UCLASS(MinimalAPI)
class UAssetReferencingPolicySubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	//~UEditorSubsystem interface
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	//~End of UEditorSubsystem interface

	// Returns whether the given asset's outgoing references are restricted in any way and should be individually validated
	UE_API bool ShouldValidateAssetReferences(const FAssetData& Asset) const;

	// Check the outgoing references of the given asset according to the asset registry and return details of any errors 
	UE_API TValueOrError<void, TArray<FAssetReferenceError>> ValidateAssetReferences(const FAssetData& Asset) const;
	UE_API TValueOrError<void, TArray<FAssetReferenceError>> ValidateAssetReferences(const FAssetData& Asset, const EAssetReferenceFilterRole Role) const;

	UE_API TSharedPtr<FDomainDatabase> GetDomainDB() const;
private:
	UE_API void UpdateDBIfNecessary() const;

	UE_API TSharedPtr<IAssetReferenceFilter> HandleMakeAssetReferenceFilter(const FAssetReferenceFilterContext& Context);

	TSharedPtr<FDomainDatabase> DomainDB;
};

#undef UE_API
