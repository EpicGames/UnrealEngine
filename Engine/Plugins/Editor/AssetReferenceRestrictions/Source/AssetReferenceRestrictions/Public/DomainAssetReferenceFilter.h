// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Editor/EditorEngine.h"
#include "Editor/AssetReferenceFilter.h"

#define UE_API ASSETREFERENCERESTRICTIONS_API

struct FAssetData;
struct FDomainDatabase;
struct FDomainData;

class FDomainAssetReferenceFilter : public IAssetReferenceFilter
{
public:
	UE_API FDomainAssetReferenceFilter(const FAssetReferenceFilterContext& Context, TSharedPtr<FDomainDatabase> InDomainDB);
	UE_API ~FDomainAssetReferenceFilter();

	//~IAssetReferenceFilter interface
	UE_API virtual bool PassesFilter(const FAssetData& AssetData, FText* OutOptionalFailureReason = nullptr) const override;
	//~End of IAssetReferenceFilter

	// Update any cached information for all filters
	static UE_API void UpdateAllFilters();
private:
	using FAssetDataInfo = TTuple<FAssetData, TSharedPtr<FDomainData>>;

	UE_API bool PassesFilterImpl(const FAssetData& AssetData, FText& OutOptionalFailureReason) const;
	UE_API bool IsCrossPluginReferenceAllowed(const FAssetDataInfo& ReferencingAssetDataInfo, const FAssetDataInfo& ReferencedAssetDataInfo) const;

	UE_API void DetermineReferencingDomain();

	/** Heuristic to find actual assets from preview assets (i.e., the material editor's preview material) */
	UE_API void TryGetAssociatedAssetsFromPossiblyPreviewObject(UObject* PossiblyPreviewObject, TArray<FAssetData>& InOutAssetsToConsider) const;

private:
	static UE_API TArray<FDomainAssetReferenceFilter*> FilterInstances;

	TSharedPtr<FDomainDatabase> DomainDB;

	TArray<FAssetReferenceFilterReferencerInfo> OriginalReferencingAssets;
	TSet<FAssetDataInfo> ReferencingAssetDataInfos;

	FText Failure_CouldNotDetermineDomain;
};

#undef UE_API
